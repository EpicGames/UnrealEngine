# -*- coding: utf-8 -*-
#
# Copyright Epic Games, Inc. All Rights Reserved.
#
# Configuration file for the Sphinx documentation builder.
#
# This file does only contain a selection of the most common options. For a
# full list see the documentation:
# http://www.sphinx-doc.org/en/stable/config

# -- Path setup --------------------------------------------------------------

# Add modules to search path
import os
import sys
sys.path.insert(0, os.path.abspath(r'..\modules'))


# -- Project information -----------------------------------------------------

project = u'Unreal Python'
copyright = u'©, Epic Games, Inc. All rights reserved. Unreal and its logo are Epic’s trademarks or registered trademarks in the US and elsewhere'
author = u'Epic Games Inc.'

# The short X.Y version
version = u'{{Version}}'
# The full version, including alpha/beta/rc tags
release = u'{{Version}} (Experimental)'


# -- General configuration ---------------------------------------------------

# If your documentation needs a minimal Sphinx version, state it here.
#
# needs_sphinx = '1.0'

# Add any Sphinx extension module names here, as strings. They can be
# extensions coming with Sphinx (named 'sphinx.ext.*') or your custom
# ones.
extensions = [
    'sphinx.ext.autodoc',
    'sphinx.ext.napoleon',
    'sphinx.ext.intersphinx'
]

# Napoleon settings
napoleon_google_docstring = True
napoleon_numpy_docstring = True
napoleon_include_init_with_doc = False
napoleon_include_private_with_doc = False
napoleon_include_special_with_doc = True

# Intersphinx settings
intersphinx_mapping = {'python': ('https://docs.python.org/2', None)}

# Add any paths that contain templates here, relative to this directory.
templates_path = ['_templates']

# The suffix(es) of source filenames.
# You can specify multiple suffix as a list of string:
#
# source_suffix = ['.rst', '.md']
source_suffix = '.rst'

# The master toctree document.
master_doc = 'index'

# The language for content autogenerated by Sphinx. Refer to documentation
# for a list of supported languages.
#
# This is also used if you do content translation via gettext catalogs.
# Usually you set "language" from the command line for these cases.
language = None

# List of patterns, relative to source directory, that match files and
# directories to ignore when looking for source files.
# This pattern also affects html_static_path and html_extra_path .
exclude_patterns = []

# The name of the Pygments (syntax highlighting) style to use.
pygments_style = 'sphinx'


# -- Options for HTML output -------------------------------------------------

# The path to look for themes
html_theme_path = ['../themes']

# The theme to use for HTML and HTML Help pages.
html_theme = 'ue_api'

# Theme options are theme-specific and customize the look and feel of a theme.
html_theme_options = {
    "rightsidebar": "false",
    "stickysidebar": "true",
    "relbarbgcolor": "#2E2E2E",
    "footerbgcolor": "#2E2E2E",
    "sidebarbgcolor": "#FFFFFF",
    "sidebartextcolor": "#000000",
    "sidebarlinkcolor": "#4b79bc",
    "th_bgcolor": "#eeeef7",
}

# Add any paths that contain custom static files (such as style sheets) here,
# relative to this directory. They are copied after the builtin static files,
# so a file named "default.css" will overwrite the builtin "default.css".
html_static_path = ['_static']

# Specify if links to the reST sources will be added to the sidebar
html_show_sourcelink = False
html_copy_source = False

# If true, the index is generated twice: once as a single page with all the
# entries, and once as one page per starting letter.
html_split_index = True

html_show_sphinx = False

# Custom sidebar templates, must be a dictionary that maps document names
# to template names.
#
# The default sidebars (for documents that don't match any pattern) are
# defined by theme itself.  Builtin themes are using these templates by
# default: ``['localtoc.html', 'relations.html', 'sourcelink.html',
# 'searchbox.html']``.
#
# html_sidebars = {}


# -- Options for HTMLHelp output ---------------------------------------------

# Output file base name for HTML help builder.
htmlhelp_basename = 'UnrealPythondoc'


# -- Options for LaTeX output ------------------------------------------------

latex_elements = {
    # The paper size ('letterpaper' or 'a4paper').
    #
    # 'papersize': 'letterpaper',

    # The font size ('10pt', '11pt' or '12pt').
    #
    # 'pointsize': '10pt',

    # Additional stuff for the LaTeX preamble.
    #
    # 'preamble': '',

    # Latex figure (float) alignment
    #
    # 'figure_align': 'htbp',
}

# Grouping the document tree into LaTeX files. List of tuples
# (source start file, target name, title,
#  author, documentclass [howto, manual, or own class]).
latex_documents = [
    (master_doc, 'UnrealPython.tex', u'UnrealPython Documentation',
     u'Epic Games', 'manual'),
]


# -- Options for manual page output ------------------------------------------

# One entry per manual page. List of tuples
# (source start file, name, description, authors, manual section).
man_pages = [
    (master_doc, 'unrealpython', u'Unreal Python Documentation',
     [author], 1)
]


# -- Options for Texinfo output ----------------------------------------------

# Grouping the document tree into Texinfo files. List of tuples
# (source start file, target name, title, author,
#  dir menu entry, description, category)
texinfo_documents = [
    (master_doc, 'UnrealPython', u'Unreal Python Documentation',
     author, 'UnrealPython', 'One line description of project.',
     'Miscellaneous'),
]


# -- Extension configuration -------------------------------------------------