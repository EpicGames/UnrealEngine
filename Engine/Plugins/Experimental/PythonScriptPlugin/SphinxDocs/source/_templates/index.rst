.. Unreal Python documentation master file
   Copyright Epic Games, Inc. All Rights Reserved.
   
Unreal Python API Documentation
===============================

.. image:: _images/python-logo.svg
  :align: left
  :width: 240 px
  :target: https://www.python.org/

.. toctree::
    :maxdepth: 4
    :caption: Getting Started
    
    introduction
    
Python API sections:

{{SectionList}}

{{TableOfContents}}
