.. Copyright Epic Games, Inc. All Rights Reserved.

Unreal Python API Introduction
===============================

.. image:: _images/python-logo.svg
  :align: center
  :target: https://www.python.org/

.. sidebar:: Python Trademarks

    *"Python" and the Python logos are trademarks or registered trademarks of the Python Software Foundation.*

The *Unreal Python API* and the associated PythonScript Plugin are used to script or automate the Unreal Editor using the `Python interpreted programming language <https://www.python.org/>`_.

See "`Scripting the Editor using Python <https://docs.unrealengine.com/en-us/ProductionPipelines/ScriptingAndAutomation/Python/index.html>`_" in the Unreal online documentation for instructions on enabling the PythonScript Plugin, tips on getting started and how to use the Python API.

.. Try to keep this page short since it may not be localized unlike the Unreal docs.
